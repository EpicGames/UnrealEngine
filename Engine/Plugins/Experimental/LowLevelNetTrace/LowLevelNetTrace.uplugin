{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Low-level network trace Plugin",
	"Description": "Actively monitors & reports system network throughput.",
	"Category": "Profiling",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"EnabledByDefault": true,
	"IsBetaVersion": true,
	"SupportedTargetPlatforms": [],
	"HasExplicitPlatforms": true,
	"Modules": [
		{
			"Name": "LowLevelNetTrace",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [],
			"HasExplicitPlatforms": true
		}
	]
}
