// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool.Rules
{
	public class LowLevelNetTrace : ModuleRules
	{
		public LowLevelNetTrace(ReadOnlyTargetRules Target) : base(Target)
        {
			PrivateDependencyModuleNames.Add("Core");
		}
	}
}
