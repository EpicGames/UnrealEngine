// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "LowLevelNetTraceModule.h"

#if UE_LOW_LEVEL_NET_TRACE_ENABLED

class ILowLevelNetTrace
{
public:
	virtual ~ILowLevelNetTrace() = default;

	virtual bool UpdateSnapshot( FLowLevelNetTraceSnapshot& OutSnapshot ) = 0;
};

#endif // UE_LOW_LEVEL_NET_TRACE_ENABLED