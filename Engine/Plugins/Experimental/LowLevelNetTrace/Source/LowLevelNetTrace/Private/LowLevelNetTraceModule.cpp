// Copyright Epic Games, Inc. All Rights Reserved.

#include "LowLevelNetTraceModule.h"

#if UE_LOW_LEVEL_NET_TRACE_ENABLED
#include "ILowLevelNetTrace.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Templates/UniquePtr.h"
#include COMPILED_PLATFORM_HEADER(LowLevelNetTrace.h)
#include <atomic>

TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(LowLevelNetTrace_UploadMbps,   TEXT("LowLevelNetTrace/UploadMbps"));
TRACE_DECLARE_UNCHECKED_FLOAT_COUNTER(LowLevelNetTrace_DownloadMbps, TEXT("LowLevelNetTrace/DownloadMbps"));


class FLowLevelNetTraceModule : public ILowLevelNetTraceModule, public FRunnable
{
public:
	virtual void StartupModule() override
	{
		LowLevelNetTrace.Reset(new FPlatformLowLevelNetTrace());
		UpdateStatistics();
		StartThread();
	}

	virtual void ShutdownModule() override
	{
		StopThread();
		LowLevelNetTrace.Reset();
	}

	virtual bool GetSnapshot( FLowLevelNetTraceSnapshot& OutSnapshot ) override
	{
		OutSnapshot = CachedNetworkSnapshot.load();
		return true;
	}

private:

	void UpdateStatistics()
	{
		// get a new snapshot
		FLowLevelNetTraceSnapshot NewSnapshot;
		if (LowLevelNetTrace->UpdateSnapshot(NewSnapshot))
		{
			//update trace stats
			TRACE_COUNTER_SET(LowLevelNetTrace_UploadMbps,    NewSnapshot.UploadMbps);
			TRACE_COUNTER_SET(LowLevelNetTrace_DownloadMbps,  NewSnapshot.DownloadMbps);

			// update cached snapshot
			CachedNetworkSnapshot.exchange(NewSnapshot);
		}
	}

	void StartThread()
	{
		check(!Thread.IsValid());
		ThreadStopEvent = FPlatformProcess::GetSynchEventFromPool(true);
		Thread.Reset( FRunnableThread::Create(this, TEXT("LowLevelLowLevelNetTrace"), 0, TPri_Lowest) );
	}

	void StopThread()
	{
		if (Thread.IsValid())
		{
			Thread.Reset();
			FPlatformProcess::ReturnSynchEventToPool(ThreadStopEvent);
			ThreadStopEvent = nullptr;
		}
	}

	// FRunnable interface
	virtual uint32 Run() override
	{
		// update the stats once per second
		while(!ThreadStopEvent->Wait(1000))
		{
			UpdateStatistics();
		}
		return 0;
	}

	virtual void Stop() override
	{
		ThreadStopEvent->Trigger();
	}
	// end of FRunnable interface


	TUniquePtr<FRunnableThread> Thread;
	FEvent* ThreadStopEvent = nullptr;

	TUniquePtr<ILowLevelNetTrace> LowLevelNetTrace;
	std::atomic<FLowLevelNetTraceSnapshot> CachedNetworkSnapshot;
};

#else
class FLowLevelNetTraceModule : public ILowLevelNetTraceModule
{
	virtual bool GetSnapshot( FLowLevelNetTraceSnapshot& OutSnapshot ) override
	{ 
		return false; 
	}
};
#endif // UE_LOW_LEVEL_NET_TRACE_ENABLED


IMPLEMENT_MODULE(FLowLevelNetTraceModule, LowLevelNetTrace)

