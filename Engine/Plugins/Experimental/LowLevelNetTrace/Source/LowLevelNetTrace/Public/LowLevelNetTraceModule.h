// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "Trace/Config.h"

#if !defined(UE_LOW_LEVEL_NET_TRACE_ENABLED)
	#define UE_LOW_LEVEL_NET_TRACE_ENABLED ((!UE_BUILD_SHIPPING) && UE_TRACE_ENABLED)
#endif

struct FLowLevelNetTraceSnapshot
{
	double UploadMbps = 0.0;
	double DownloadMbps = 0.0;
	double TimeStamp = 0.0;
};

class ILowLevelNetTraceModule : public IModuleInterface
{
public:
	static inline ILowLevelNetTraceModule& Get()
	{
		return FModuleManager::LoadModuleChecked<ILowLevelNetTraceModule>("LowLevelNetTrace");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("LowLevelNetTrace");
	}

	virtual bool GetSnapshot( FLowLevelNetTraceSnapshot& OutSnapshot ) = 0;
};
